{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "OpenXRViveTracker",
	"Description": "OpenXR Vive Tracker provides XR_HTCX_vive_tracker_interaction.",
	"Category": "Virtual Reality",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Android"
	],
	"Modules": [
		{
			"Name": "OpenXRViveTracker",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Android"
			]
		}
	],
	"Plugins": [
		{
			"Name": "OpenXR",
			"Enabled": true
		}
	]
}