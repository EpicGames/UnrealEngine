// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class OpenXRViveTracker : ModuleRules
	{
		public OpenXRViveTracker(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"InputDevice"
				}
			 );

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"HeadMountedDisplay",
					"XRBase",
					"InputCore",
					"OpenXRHMD",
					"OpenXRInput",
					"Slate",
					"SlateCore",
					"ApplicationCore"
				}
			);

			PrivateIncludePathModuleNames.Add("OpenXR");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenXR");

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}
