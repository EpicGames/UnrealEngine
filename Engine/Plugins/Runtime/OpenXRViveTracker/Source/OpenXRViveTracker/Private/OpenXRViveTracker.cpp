// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenXRViveTracker.h"
#include "IOpenXRViveTrackerModule.h"

#include "IXRTrackingSystem.h"
#include "CoreMinimal.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Engine.h"
#include "OpenXRCore.h"
#include "Framework/Application/SlateApplication.h"
#include "IOpenXRExtensionPlugin.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "HeadMountedDisplayFunctionLibrary.h"
#include "IOpenXRHMDModule.h"
#include "IOpenXRHMD.h"
#include "IOpenXRExtensionPluginDelegates.h"

#define LOCTEXT_NAMESPACE "OpenXRViveTracker"

DEFINE_LOG_CATEGORY_STATIC(LogOpenXRViveTracker, Display, All);

class FOpenXRViveTrackerModule :
	public IOpenXRViveTrackerModule,
	public IOpenXRExtensionPlugin
{
public:
	FOpenXRViveTrackerModule()
		: InputDevice(nullptr)
	{}

	virtual void StartupModule() override
	{
		IOpenXRViveTrackerModule::StartupModule();
		RegisterOpenXRExtensionModularFeature();

		// HACK: Generic Application might not be instantiated at this point so we create the input device with a
		// dummy message handler. When the Generic Application creates the input device it passes a valid message
		// handler to it which is further on used for all the controller events. This hack fixes issues caused by
		// using a custom input device before the Generic Application has instantiated it. Eg. within BeginPlay()
		//
		// This also fixes the warnings that pop up on the custom input keys when the blueprint loads. Those
		// warnings are caused because Unreal loads the bluerints before the input device has been instantiated
		// and has added its keys, thus leading Unreal to believe that those keys don't exist. This hack causes
		// an earlier instantiation of the input device, and consequently, the custom keys.
		TSharedPtr<FGenericApplicationMessageHandler> DummyMessageHandler(new FGenericApplicationMessageHandler());
		CreateInputDevice(DummyMessageHandler.ToSharedRef());
	}

	virtual void ShutdownModule() override
	{
		IOpenXRViveTrackerModule::ShutdownModule();
	}

	virtual TSharedPtr<class IInputDevice> CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override
	{
		if (!InputDevice.IsValid())
		{
			TSharedPtr<FOpenXRViveTracker> ViveTrackingInputDevice(new FOpenXRViveTracker(InMessageHandler));
			InputDevice = ViveTrackingInputDevice;

			return InputDevice;
		}
		else
		{
			InputDevice.Get()->SetMessageHandler(InMessageHandler);
			return InputDevice;
		}
	}

	virtual TSharedPtr<IInputDevice> GetInputDevice() override
	{
		if (!InputDevice.IsValid())
		{
			CreateInputDevice(FSlateApplication::Get().GetPlatformApplication()->GetMessageHandler());
		}
		return InputDevice;
	}

private:
	TSharedPtr<FOpenXRViveTracker> InputDevice;
};

IMPLEMENT_MODULE(FOpenXRViveTrackerModule, OpenXRViveTracker);

FOpenXRViveTracker::FViveTracker::FViveTracker(XrActionSet InActionSet, FOpenXRPath InRolePath, const char* InName)
	: ActionSet(InActionSet)
	, RolePath(InRolePath)
	, GripAction(XR_NULL_HANDLE)
	, VibrationAction(XR_NULL_HANDLE)
	, DeviceId(-1)
{
	XrActionCreateInfo Info;
	Info.type = XR_TYPE_ACTION_CREATE_INFO;
	Info.next = nullptr;
	Info.countSubactionPaths = 0;
	Info.subactionPaths = nullptr;

	FCStringAnsi::Strncpy(Info.localizedActionName, InName, XR_MAX_ACTION_NAME_SIZE);
	FCStringAnsi::StrncatTruncateDest(Info.localizedActionName, XR_MAX_ACTION_NAME_SIZE, " Pose");
	FilterActionName(Info.localizedActionName, Info.actionName);
	Info.actionType = XR_ACTION_TYPE_POSE_INPUT;
	XR_ENSURE(xrCreateAction(ActionSet, &Info, &GripAction));

	FCStringAnsi::Strncpy(Info.localizedActionName, InName, XR_MAX_ACTION_NAME_SIZE);
	FCStringAnsi::StrncatTruncateDest(Info.localizedActionName, XR_MAX_ACTION_NAME_SIZE, " Vibration");
	FilterActionName(Info.localizedActionName, Info.actionName);
	Info.actionType = XR_ACTION_TYPE_VIBRATION_OUTPUT;
	XR_ENSURE(xrCreateAction(ActionSet, &Info, &VibrationAction));
}

void FOpenXRViveTracker::FViveTracker::AddTrackedDevices(IOpenXRHMD* HMD)
{
	if (HMD)
	{
		DeviceId = HMD->AddTrackedDevice(GripAction, RolePath);
	}
}

void FOpenXRViveTracker::FViveTracker::GetSuggestedBindings(TArray<XrActionSuggestedBinding>& OutSuggestedBindings)
{
	OutSuggestedBindings.Add(XrActionSuggestedBinding{ GripAction, RolePath / FString("input/grip/pose") });
	OutSuggestedBindings.Add(XrActionSuggestedBinding{ VibrationAction, RolePath / FString("output/haptic") });
}

FOpenXRViveTracker::FOpenXRViveTracker(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
	: MessageHandler(InMessageHandler)
	, DeviceIndex(0)
	, TrackerActionSet(XR_NULL_HANDLE)
	, Trackers()
{
	// Register modular feature manually
	IModularFeatures::Get().RegisterModularFeature(IMotionController::GetModularFeatureName(), static_cast<IMotionController*>(this));
	IModularFeatures::Get().RegisterModularFeature(IOpenXRExtensionPlugin::GetModularFeatureName(), static_cast<IOpenXRExtensionPlugin*>(this));
	AddKeys();

	// We're implicitly requiring that the OpenXRPlugin has been loaded and
	// initialized at this point.
	if (!IOpenXRHMDModule::Get().IsAvailable())
	{
		UE_LOGF(LogOpenXRViveTracker, Error, "Error - OpenXRHMDPlugin isn't available");
	}

	MotionSourceToEControllerHandMap.Add(TEXT("Pad"), EControllerHand::Pad);
	MotionSourceToEControllerHandMap.Add(TEXT("ExternalCamera"), EControllerHand::ExternalCamera);
	MotionSourceToEControllerHandMap.Add(TEXT("Gun"), EControllerHand::Gun);
	MotionSourceToEControllerHandMap.Add(TEXT("Chest"), EControllerHand::Chest);
	MotionSourceToEControllerHandMap.Add(TEXT("LeftShoulder"), EControllerHand::LeftShoulder);
	MotionSourceToEControllerHandMap.Add(TEXT("RightShoulder"), EControllerHand::RightShoulder);
	MotionSourceToEControllerHandMap.Add(TEXT("LeftElbow"), EControllerHand::LeftElbow);
	MotionSourceToEControllerHandMap.Add(TEXT("RightElbow"), EControllerHand::RightElbow);
	MotionSourceToEControllerHandMap.Add(TEXT("Waist"), EControllerHand::Waist);
	MotionSourceToEControllerHandMap.Add(TEXT("LeftKnee"), EControllerHand::LeftKnee);
	MotionSourceToEControllerHandMap.Add(TEXT("RightKnee"), EControllerHand::RightKnee);
	MotionSourceToEControllerHandMap.Add(TEXT("LeftFoot"), EControllerHand::LeftFoot);
	MotionSourceToEControllerHandMap.Add(TEXT("RightFoot"), EControllerHand::RightFoot);
}

FOpenXRViveTracker::~FOpenXRViveTracker()
{
	// Unregister modular feature manually
	IModularFeatures::Get().UnregisterModularFeature(IMotionController::GetModularFeatureName(), static_cast<IMotionController*>(this));
	IModularFeatures::Get().UnregisterModularFeature(IOpenXRExtensionPlugin::GetModularFeatureName(), static_cast<IOpenXRExtensionPlugin*>(this));
}

bool FOpenXRViveTracker::GetRequiredExtensions(TArray<const ANSICHAR*>& OutExtensions)
{
	OutExtensions.Add(XR_HTCX_VIVE_TRACKER_INTERACTION_EXTENSION_NAME);
	return true;
}

void FOpenXRViveTracker::PostCreateInstance(XrInstance InInstance)
{
	// Store extension open xr calls to member function pointers for convenient use.
	XR_ENSURE(xrGetInstanceProcAddr(InInstance, "xrEnumerateViveTrackerPathsHTCX", (PFN_xrVoidFunction*)&xrEnumerateViveTrackerPathsHTCX));
}

const void* FOpenXRViveTracker::OnCreateSession(XrInstance InInstance, XrSystemId InSystem, const void* InNext)
{
	if (GEngine && GEngine->XRSystem.IsValid())
	{
		XRTrackingSystem = GEngine->XRSystem.Get();
		OpenXRHMD = XRTrackingSystem->GetIOpenXRHMD();
	}

	if (TrackerActionSet)
	{
		XR_ENSURE(xrDestroyActionSet(TrackerActionSet));
	}

	XrActionSetCreateInfo CreateInfo = { XR_TYPE_ACTION_SET_CREATE_INFO };
	FCStringAnsi::Strncpy(CreateInfo.actionSetName, "trackers", XR_MAX_ACTION_SET_NAME_SIZE);
	FCStringAnsi::Strncpy(CreateInfo.localizedActionSetName, "Vive Trackers",
		XR_MAX_LOCALIZED_ACTION_SET_NAME_SIZE);
	XR_ENSURE(xrCreateActionSet(InInstance, &CreateInfo, &TrackerActionSet));

	Trackers.Add(EControllerHand::Pad, FViveTracker(TrackerActionSet, FOpenXRPath("/user/vive_tracker_htcx/role/keyboard"), "Keyboard"));
	Trackers.Add(EControllerHand::ExternalCamera, FViveTracker(TrackerActionSet, FOpenXRPath("/user/vive_tracker_htcx/role/camera"), "External Camera"));
	Trackers.Add(EControllerHand::Gun, FViveTracker(TrackerActionSet, FOpenXRPath("/user/vive_tracker_htcx/role/handheld_object"), "Handheld Object"));
	Trackers.Add(EControllerHand::Chest, FViveTracker(TrackerActionSet, FOpenXRPath("/user/vive_tracker_htcx/role/chest"), "Chest"));
	Trackers.Add(EControllerHand::LeftShoulder, FViveTracker(TrackerActionSet, FOpenXRPath("/user/vive_tracker_htcx/role/left_shoulder"), "Left Shoulder"));
	Trackers.Add(EControllerHand::RightShoulder, FViveTracker(TrackerActionSet, FOpenXRPath("/user/vive_tracker_htcx/role/right_shoulder"), "Right Shoulder"));
	Trackers.Add(EControllerHand::LeftElbow, FViveTracker(TrackerActionSet, FOpenXRPath("/user/vive_tracker_htcx/role/left_elbow"), "Left Elbow"));
	Trackers.Add(EControllerHand::RightElbow, FViveTracker(TrackerActionSet, FOpenXRPath("/user/vive_tracker_htcx/role/right_elbow"), "Right Elbow"));
	Trackers.Add(EControllerHand::Waist, FViveTracker(TrackerActionSet, FOpenXRPath("/user/vive_tracker_htcx/role/waist"), "Waist"));
	Trackers.Add(EControllerHand::LeftKnee, FViveTracker(TrackerActionSet, FOpenXRPath("/user/vive_tracker_htcx/role/left_knee"), "Left Knee"));
	Trackers.Add(EControllerHand::RightKnee, FViveTracker(TrackerActionSet, FOpenXRPath("/user/vive_tracker_htcx/role/right_knee"), "Right Knee"));
	Trackers.Add(EControllerHand::LeftFoot, FViveTracker(TrackerActionSet, FOpenXRPath("/user/vive_tracker_htcx/role/left_foot"), "Left Foot"));
	Trackers.Add(EControllerHand::RightFoot, FViveTracker(TrackerActionSet, FOpenXRPath("/user/vive_tracker_htcx/role/right_foot"), "Right Foot"));

	TArray<XrActionSuggestedBinding> Bindings;
	for (TPair<EControllerHand, FViveTracker> Tracker : Trackers)
	{
		Tracker.Value.GetSuggestedBindings(Bindings);
	}

	uint32_t PathCount = 0;
	TArray<XrViveTrackerPathsHTCX> TrackerPaths;
	XR_ENSURE(xrEnumerateViveTrackerPathsHTCX(InInstance, 0, &PathCount, nullptr));
	TrackerPaths.SetNum(PathCount);
	XR_ENSURE(xrEnumerateViveTrackerPathsHTCX(InInstance, PathCount, &PathCount, TrackerPaths.GetData()));
	check(TrackerPaths.Num() == PathCount);

	// Add LiveLink poses for trackers that have no role assigned
	uint32 TrackerIndex = 0;
	for (const XrViveTrackerPathsHTCX& TrackerPath : TrackerPaths) //-V1078
	{
		if (TrackerPath.rolePath == XR_NULL_PATH)
		{
			char Name[XR_MAX_ACTION_NAME_SIZE];
			FCStringAnsi::Snprintf(Name, XR_MAX_ACTION_NAME_SIZE, "Tracker %u", TrackerIndex++);
			FViveTracker Tracker(TrackerActionSet, TrackerPath.persistentPath, Name);
			UnassignedTrackers.Add(Tracker);
			Tracker.GetSuggestedBindings(Bindings);
		}
	}

	XrInteractionProfileSuggestedBinding InteractionProfile = { XR_TYPE_INTERACTION_PROFILE_SUGGESTED_BINDING };
	InteractionProfile.interactionProfile = FOpenXRPath("/interaction_profiles/htc/vive_tracker_htcx");
	InteractionProfile.countSuggestedBindings = Bindings.Num();
	InteractionProfile.suggestedBindings = Bindings.GetData();
	XR_ENSURE(xrSuggestInteractionProfileBindings(InInstance, &InteractionProfile));

	return InNext;
}

void FOpenXRViveTracker::OnDestroySession(XrSession InSession)
{
	Trackers.Reset();
	UnassignedTrackers.Reset();
	bActionsAttached = false;
}

void FOpenXRViveTracker::AttachActionSets(TSet<XrActionSet>& OutActionSets)
{
	for (TPair<EControllerHand, FViveTracker>& Tracker : Trackers)
	{
		Tracker.Value.AddTrackedDevices(OpenXRHMD);
	}

	for (FViveTracker& Tracker : UnassignedTrackers)
	{
		Tracker.AddTrackedDevices(OpenXRHMD);
	}

	OutActionSets.Add(TrackerActionSet);

	bActionsAttached = true;
}

void FOpenXRViveTracker::GetActiveActionSetsForSync(TArray<XrActiveActionSet>& OutActiveSets)
{
	XrActiveActionSet ActiveTrackerSet{ TrackerActionSet, XR_NULL_PATH };
	OutActiveSets.Add(ActiveTrackerSet);
}

bool FOpenXRViveTracker::GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, float WorldToMetersScale) const
{
	if (!bActionsAttached || OpenXRHMD == nullptr)
	{
		return false;
	}

	if (ControllerIndex == DeviceIndex && MotionSourceToEControllerHandMap.Contains(MotionSource))
	{
		FQuat Orientation;
		bool Success = XRTrackingSystem->GetCurrentPose(GetDeviceIDForMotionSource(MotionSource), Orientation, OutPosition);
		OutOrientation = FRotator(Orientation);
		return Success;
	}
	return false;
}

bool FOpenXRViveTracker::GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, bool& OutbProvidedLinearVelocity, FVector& OutLinearVelocity, bool& OutbProvidedAngularVelocity, FVector& OutAngularVelocityAsAxisAndLength, bool& OutbProvidedLinearAcceleration, FVector& OutLinearAcceleration, float WorldToMetersScale) const
{
	// FTimespan initializes to 0 and GetControllerOrientationAndPositionForTime with time 0 will return the latest data.
	FTimespan Time;
	bool OutTimeWasUsed = false;
	return GetControllerOrientationAndPositionForTime(ControllerIndex, MotionSource, Time, OutTimeWasUsed, OutOrientation, OutPosition, OutbProvidedLinearVelocity, OutLinearVelocity, OutbProvidedAngularVelocity, OutAngularVelocityAsAxisAndLength, OutbProvidedLinearAcceleration, OutLinearAcceleration, WorldToMetersScale);
}

bool FOpenXRViveTracker::GetControllerOrientationAndPositionForTime(const int32 ControllerIndex, const FName MotionSource, FTimespan Time, bool& OutTimeWasUsed, FRotator& OutOrientation, FVector& OutPosition, bool& OutbProvidedLinearVelocity, FVector& OutLinearVelocity, bool& OutbProvidedAngularVelocity, FVector& OutAngularVelocityAsAxisAndLength, bool& OutbProvidedLinearAcceleration, FVector& OutLinearAcceleration, float WorldToMetersScale) const
{
	if (OpenXRHMD && ControllerIndex == DeviceIndex && MotionSourceToEControllerHandMap.Contains(MotionSource))
	{
		FQuat Orientation;
		bool Success = OpenXRHMD->GetPoseForTime(GetDeviceIDForMotionSource(MotionSource), Time, OutTimeWasUsed, Orientation, OutPosition, OutbProvidedLinearVelocity, OutLinearVelocity, OutbProvidedAngularVelocity, OutAngularVelocityAsAxisAndLength, OutbProvidedLinearAcceleration, OutLinearAcceleration, WorldToMetersScale);
		OutOrientation = FRotator(Orientation);
		return Success;
	}
	return false;
}

ETrackingStatus FOpenXRViveTracker::GetControllerTrackingStatus(const int32 ControllerIndex, const FName MotionSource) const
{
	if (!bActionsAttached || OpenXRHMD == nullptr)
	{
		return ETrackingStatus::NotTracked;
	}

	XrSession Session = OpenXRHMD->GetSession();
	if (Session == XR_NULL_HANDLE)
	{
		return ETrackingStatus::NotTracked;
	}

	if (!(ControllerIndex == DeviceIndex && MotionSourceToEControllerHandMap.Contains(MotionSource)))
	{
		return ETrackingStatus::NotTracked;
	}

	const EControllerHand DeviceHand = MotionSourceToEControllerHandMap.FindChecked(MotionSource);

	const FViveTracker* Tracker = Trackers.Find(DeviceHand);
	if (!Tracker)
	{
		return ETrackingStatus::NotTracked;
	}

	XrActionStateGetInfo Info = { XR_TYPE_ACTION_STATE_GET_INFO };
	Info.action = Tracker->GripAction;
	Info.subactionPath = XR_NULL_PATH;

	XrActionStatePose State = { XR_TYPE_ACTION_STATE_POSE };
	if (!XR_ENSURE(xrGetActionStatePose(Session, &Info, &State)))
	{
		return ETrackingStatus::NotTracked;
	}
	return State.isActive ? ETrackingStatus::Tracked : ETrackingStatus::NotTracked;
}

FName FOpenXRViveTracker::GetMotionControllerDeviceTypeName() const
{
	const static FName DefaultName(TEXT("OpenXRViveTracker"));
	return DefaultName;
}

void FOpenXRViveTracker::EnumerateSources(TArray<FMotionControllerSource>& SourcesOut) const
{
	check(IsInGameThread());

	TArray<FName> Sources;
	MotionSourceToEControllerHandMap.GenerateKeyArray(Sources);
	SourcesOut.Append(Sources);
}

void FOpenXRViveTracker::Tick(float DeltaTime)
{
}

void FOpenXRViveTracker::SendControllerEvents()
{
}

void FOpenXRViveTracker::SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
	MessageHandler = InMessageHandler;
}

bool FOpenXRViveTracker::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	return false;
}

void FOpenXRViveTracker::SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
{
	// Large channel type maps to amplitude. We are interested in amplitude.
	if ((ChannelType == FForceFeedbackChannelType::LEFT_LARGE) ||
		(ChannelType == FForceFeedbackChannelType::RIGHT_LARGE))
	{
		FHapticFeedbackValues Values(XR_FREQUENCY_UNSPECIFIED, Value);
		SetHapticFeedbackValues(ControllerId, ChannelType == FForceFeedbackChannelType::LEFT_LARGE ? (int32)EControllerHand::Left : (int32)EControllerHand::Right, Values);
	}
}

void FOpenXRViveTracker::SetChannelValues(int32 ControllerId, const FForceFeedbackValues& values)
{
	FHapticFeedbackValues leftHaptics = FHapticFeedbackValues(
		values.LeftSmall,		// frequency
		values.LeftLarge);		// amplitude
	FHapticFeedbackValues rightHaptics = FHapticFeedbackValues(
		values.RightSmall,		// frequency
		values.RightLarge);		// amplitude

	SetHapticFeedbackValues(
		ControllerId,
		(int32)EControllerHand::Left,
		leftHaptics);

	SetHapticFeedbackValues(
		ControllerId,
		(int32)EControllerHand::Right,
		rightHaptics);
}

bool FOpenXRViveTracker::IsGamepadAttached() const
{
	return false;
}

// TODO: Refactor API to change the Hand type to EControllerHand
void FOpenXRViveTracker::SetHapticFeedbackValues(int32 ControllerId, int32 Hand, const FHapticFeedbackValues& Values)
{
	if (!bActionsAttached || OpenXRHMD == nullptr)
	{
		return;
	}

	XrSession Session = OpenXRHMD->GetSession();

	if (Session == XR_NULL_HANDLE)
	{
		return;
	}

	if (!OpenXRHMD->IsFocused())
	{
		return;
	}

	XrHapticVibration HapticValue;
	HapticValue.type = XR_TYPE_HAPTIC_VIBRATION;
	HapticValue.next = nullptr;
	HapticValue.duration = XR_INFINITE_DURATION;
	HapticValue.frequency = Values.Frequency;
	HapticValue.amplitude = Values.Amplitude;

	FViveTracker* Tracker = Trackers.Find((EControllerHand)Hand);
	if (Tracker && ControllerId == DeviceIndex)
	{
		XrHapticActionInfo HapticActionInfo;
		HapticActionInfo.type = XR_TYPE_HAPTIC_ACTION_INFO;
		HapticActionInfo.next = nullptr;
		HapticActionInfo.subactionPath = XR_NULL_PATH;
		HapticActionInfo.action = Tracker->VibrationAction;
		if (Values.HapticBuffer == nullptr && (Values.Amplitude <= 0.0f || Values.Frequency < XR_FREQUENCY_UNSPECIFIED))
		{
			XR_ENSURE(xrStopHapticFeedback(Session, &HapticActionInfo));
		}
		else
		{
			FOpenXRExtensionChainStructPtrs ScopedExtensionChainStructs;
			if (Values.HapticBuffer != nullptr)
			{
				OpenXRHMD->GetIOpenXRExtensionPluginDelegates().GetApplyHapticFeedbackAddChainStructsDelegate().Broadcast(&HapticValue, ScopedExtensionChainStructs, Values.HapticBuffer);
			}
			XR_ENSURE(xrApplyHapticFeedback(Session, &HapticActionInfo, (const XrHapticBaseHeader*)&HapticValue));
		}
	}
}

void FOpenXRViveTracker::GetHapticFrequencyRange(float& MinFrequency, float& MaxFrequency) const
{
	MinFrequency = XR_FREQUENCY_UNSPECIFIED;
	MaxFrequency = XR_FREQUENCY_UNSPECIFIED;
}

float FOpenXRViveTracker::GetHapticAmplitudeScale() const
{
	return 1.0f;
}

void FOpenXRViveTracker::AddKeys()
{
}

XrAction FOpenXRViveTracker::GetActionForMotionSource(FName MotionSource) const
{
	const EControllerHand* Hand = MotionSourceToEControllerHandMap.Find(MotionSource);
	if (Hand == nullptr || Trackers.IsEmpty())
	{
		return XR_NULL_HANDLE;
	}
	const FViveTracker& Tracker = Trackers[*Hand];
	return Tracker.GripAction;
}

int32 FOpenXRViveTracker::GetDeviceIDForMotionSource(FName MotionSource) const
{
	if (Trackers.IsEmpty())
	{
		return -1;
	}
	const FViveTracker& Tracker = Trackers[MotionSourceToEControllerHandMap.FindChecked(MotionSource)];
	return Tracker.DeviceId;
}

XrPath FOpenXRViveTracker::GetRolePathForMotionSource(FName MotionSource) const
{
	if (Trackers.IsEmpty())
	{
		return XR_NULL_PATH;
	}
	const FViveTracker& Tracker = Trackers[MotionSourceToEControllerHandMap.FindChecked(MotionSource)];
	return Tracker.RolePath;
}

#undef LOCTEXT_NAMESPACE
