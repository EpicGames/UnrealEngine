// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/IInputInterface.h"
#include "HeadMountedDisplayTypes.h"
#include "XRMotionControllerBase.h"
#include "InputCoreTypes.h"
#include "IInputDevice.h"
#include "IHapticDevice.h"

#include "IOpenXRExtensionPlugin.h"
#include "OpenXRCore.h"

/**
  * OpenXR ViveTracker
  */
class FOpenXRViveTracker :
	public IOpenXRExtensionPlugin,
	public IInputDevice,
	public FXRMotionControllerBase,
	public IHapticDevice
{
public:
	struct FViveTracker
	{
		XrActionSet		ActionSet;
		FOpenXRPath		RolePath;
		XrAction		GripAction;
		XrAction		VibrationAction;
		int32			DeviceId;

		FViveTracker(XrActionSet InActionSet, FOpenXRPath InRolePath, const char* InName);

		void AddTrackedDevices(class IOpenXRHMD* HMD);
		void GetSuggestedBindings(TArray<XrActionSuggestedBinding>& OutSuggestedBindings);
	};

public:
	FOpenXRViveTracker(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler);
	virtual ~FOpenXRViveTracker();

	/** IOpenXRExtensionPlugin */
	virtual FString GetDisplayName() override
	{
		return FString(TEXT("OpenXRViveTracker"));
	}
	virtual bool GetRequiredExtensions(TArray<const ANSICHAR*>& OutExtensions) override;
	virtual void PostCreateInstance(XrInstance InInstance) override;
	virtual const void* OnCreateSession(XrInstance InInstance, XrSystemId InSystem, const void* InNext) override;
	virtual void OnDestroySession(XrSession InSession) override;
	virtual void AttachActionSets(TSet<XrActionSet>& OutActionSets) override;
	virtual void GetActiveActionSetsForSync(TArray<XrActiveActionSet>& OutActiveSets) override;

	/** IMotionController interface */
	virtual bool GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, float WorldToMetersScale) const override;
	virtual bool GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, bool& OutbProvidedLinearVelocity, FVector& OutLinearVelocity, bool& OutbProvidedAngularVelocity, FVector& OutAngularVelocityAsAxisAndLength, bool& OutbProvidedLinearAcceleration, FVector& OutLinearAcceleration, float WorldToMetersScale) const override;
	virtual bool GetControllerOrientationAndPositionForTime(const int32 ControllerIndex, const FName MotionSource, FTimespan Time, bool& OutTimeWasUsed, FRotator& OutOrientation, FVector& OutPosition, bool& OutbProvidedLinearVelocity, FVector& OutLinearVelocity, bool& OutbProvidedAngularVelocity, FVector& OutAngularVelocityAsAxisAndLength, bool& OutbProvidedLinearAcceleration, FVector& OutLinearAcceleration, float WorldToMetersScale) const override;
	virtual ETrackingStatus GetControllerTrackingStatus(const int32 ControllerIndex, const FName MotionSource) const override;
	virtual FName GetMotionControllerDeviceTypeName() const override;
	virtual void EnumerateSources(TArray<FMotionControllerSource>& SourcesOut) const override;

	/** IInputDevice interface */
	virtual void Tick(float DeltaTime) override;
	virtual void SendControllerEvents() override;
	virtual void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override;
	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	virtual void SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override;
	virtual void SetChannelValues(int32 ControllerId, const FForceFeedbackValues &values) override;
	virtual bool SupportsForceFeedback(int32 ControllerId) override { return true; }
	virtual bool IsGamepadAttached() const override;

	// IHapticDevice overrides
	IHapticDevice* GetHapticDevice() override { return (IHapticDevice*)this; }
	virtual void SetHapticFeedbackValues(int32 ControllerId, int32 Hand, const FHapticFeedbackValues& Values) override;

	virtual void GetHapticFrequencyRange(float& MinFrequency, float& MaxFrequency) const override;
	virtual float GetHapticAmplitudeScale() const override;

private:
	void AddKeys();

	XrAction GetActionForMotionSource(FName MotionSource) const;
	int32 GetDeviceIDForMotionSource(FName MotionSource) const;
	XrPath GetRolePathForMotionSource(FName MotionSource) const;

	bool bActionsAttached = false;

	PFN_xrEnumerateViveTrackerPathsHTCX xrEnumerateViveTrackerPathsHTCX = nullptr;

	class IXRTrackingSystem* XRTrackingSystem = nullptr;
	class IOpenXRHMD* OpenXRHMD = nullptr;

	TSharedPtr<FGenericApplicationMessageHandler> MessageHandler;
	int32 DeviceIndex;

	XrActionSet TrackerActionSet;
	TMap<EControllerHand, FViveTracker> Trackers;
	TArray<FViveTracker> UnassignedTrackers;

	TMap<FName, EControllerHand> MotionSourceToEControllerHandMap;
};
